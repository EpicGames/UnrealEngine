{
    "FileVersion" : 3,

    "Version" : 11,
    "VersionName" : "1.18.0",
    "FriendlyName" : "Google ARCore Services",
    "Description" : "Provide functionality in Google cross-platform ARCore services.",
    "Category" : "Augmented Reality",
    "CreatedBy" : "Google",
    "CreatedByURL" : "https://developers.google.com/ar/",
    "CanContainContent": true,
    "IsBetaVersion": false,
    "EnabledByDefault": false,
    "Modules" :
    [
        {
            "Name" : "GoogleARCoreServices",
            "Type" : "Runtime",
            "PlatformAllowList": [ "Win64", "Mac", "Android", "Linux", "IOS"]
        },
    ],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		}
	]
}
