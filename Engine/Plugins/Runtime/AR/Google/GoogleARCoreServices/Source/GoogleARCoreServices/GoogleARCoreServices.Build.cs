// Copyright Epic Games, Inc. All Rights Reserved.


using System.IO;


namespace UnrealBuildTool.Rules
{
	public class GoogleARCoreServices : ModuleRules
	{
		public GoogleARCoreServices(ReadOnlyTargetRules Target) : base(Target)
		{			
			PublicDependencyModuleNames.AddRange(
					new string[]
					{
						"HeadMountedDisplay",
						"AugmentedReality",
					}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"GoogleARCoreSDK",
					"XRBase",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"Settings" // For editor settings panel.
				}
			);


			if (Target.Platform == UnrealTargetPlatform.Android)
			{
                // Register Plugin Language
                string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
                AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "GoogleARCoreServices_APL.xml"));
			}
		}
	}
}
