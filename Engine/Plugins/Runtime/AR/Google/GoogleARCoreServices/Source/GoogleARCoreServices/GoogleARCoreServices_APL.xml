<?xml version="1.0" encoding="utf-8"?>
<!--GoogleARCore plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
  <!-- init section is always evaluated once per architecture -->
  <init>
    <log text="GoogleARServices APL init"/>
    <setStringFromProperty result="APIKey" ini="Engine" section="/Script/GoogleARCoreServices.GoogleARCoreServicesEditorSettings" property="AndroidAPIKey" default=""/>
	</init>

  <!-- optional updates applied to AndroidManifest.xml -->
  <androidManifestUpdates>
    <addPermission android:name="android.permission.INTERNET"/>
    <!-- Add the API Key for GoogleARCore Services -->
    <setElement result="ARCoreAPIKey" value="meta-data" />
    <addAttribute tag="$ARCoreAPIKey" name="android:name" value="com.google.android.ar.API_KEY" />
    <addAttribute tag="$ARCoreAPIKey" name="android:value" value="$S(APIKey)" />
    <addElement tag="application" name="ARCoreAPIKey" />
  </androidManifestUpdates>

</root>
