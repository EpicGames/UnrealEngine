// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreCloudARPinManager.h"
#include "IXRTrackingSystem.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#elif PLATFORM_IOS
#include "Misc/ConfigCacheIni.h"
#include "Misc/EngineVersion.h"
#include "arcore_ios_c_api.h"
#endif

#if ARCORE_SERVICE_SUPPORTED_PLATFORM
#include "IXRTrackingSystem.h"
#endif

namespace {
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	EARPinCloudTaskResult ToCloudTaskResult(ArStatus Status)
	{
		switch (Status)
		{
		case AR_SUCCESS:
			return EARPinCloudTaskResult::Started;
		case AR_ERROR_NOT_TRACKING:
			return EARPinCloudTaskResult::NotTracking;
		case AR_ERROR_ANCHOR_NOT_SUPPORTED_FOR_HOSTING:
			return EARPinCloudTaskResult::InvalidPin;
#if PLATFORM_ANDROID
		case AR_ERROR_SESSION_PAUSED:
			return EARPinCloudTaskResult::SessionPaused;
		case AR_ERROR_CLOUD_ANCHORS_NOT_CONFIGURED:
			return EARPinCloudTaskResult::CloudARPinNotEnabled;
		case AR_ERROR_RESOURCE_EXHAUSTED:
			return EARPinCloudTaskResult::ResourceExhausted;
#endif
		default:
			UE_LOGF(LogGoogleARCoreServices, Warning, "ToCloudTaskResult defaulting to EARPinCloudTaskResult::Failed for ArStatus %d", Status);
			ensureAlwaysMsgf(false, TEXT("Unknown conversion from ArStatus %d to EARPinCloudTaskResult"), Status);
			return EARPinCloudTaskResult::Failed;
		}
	}

	ECloudARPinCloudState ToARPinCloudState(ArCloudAnchorState State)
	{
		switch (State)
		{
		// Note: ECloudARPinCloudState::InProgress and ECloudARPinCloudState::Cancelled result from an ArFutureState, not 
		// an ArCloudAnchorState, so this function does not ever return those values.
		case AR_CLOUD_ANCHOR_STATE_NONE:
			return ECloudARPinCloudState::NotHosted;
#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
		case AR_CLOUD_ANCHOR_STATE_TASK_IN_PROGRESS:
			return ECloudARPinCloudState::InProgress;
#endif
		case AR_CLOUD_ANCHOR_STATE_SUCCESS:
			return ECloudARPinCloudState::Success;
		case AR_CLOUD_ANCHOR_STATE_ERROR_INTERNAL:
			return ECloudARPinCloudState::ErrorInternalError;
		case AR_CLOUD_ANCHOR_STATE_ERROR_NOT_AUTHORIZED:
			return ECloudARPinCloudState::ErrorNotAuthorized;
		case AR_CLOUD_ANCHOR_STATE_ERROR_RESOURCE_EXHAUSTED:
			return ECloudARPinCloudState::ErrorResourceExhausted;
		case AR_CLOUD_ANCHOR_STATE_ERROR_HOSTING_DATASET_PROCESSING_FAILED:
			return ECloudARPinCloudState::ErrorHostingDatasetProcessingFailed;
		case AR_CLOUD_ANCHOR_STATE_ERROR_CLOUD_ID_NOT_FOUND:
			return ECloudARPinCloudState::ErrorResolvingCloudIDNotFound;
		case AR_CLOUD_ANCHOR_STATE_ERROR_RESOLVING_SDK_VERSION_TOO_OLD:
			return ECloudARPinCloudState::ErrorSDKVersionTooOld;
		case AR_CLOUD_ANCHOR_STATE_ERROR_RESOLVING_SDK_VERSION_TOO_NEW:
			return ECloudARPinCloudState::ErrorSDKVersionTooNew;
		case AR_CLOUD_ANCHOR_STATE_ERROR_HOSTING_SERVICE_UNAVAILABLE:
			return ECloudARPinCloudState::ErrorServiceUnavailable;
		default:
			ensureAlwaysMsgf(false, TEXT("Unknown value when attempting conversion from ArCloudAnchorState %d to ECloudARPinCloudState"), State);
			return ECloudARPinCloudState::ErrorInternalError;
		}
	}

	const TCHAR* ToTCHAR(ArCloudAnchorState State)
	{
		switch (State)
		{
		case AR_CLOUD_ANCHOR_STATE_NONE:
			return TEXT("AR_CLOUD_ANCHOR_STATE_NONE");
		case AR_CLOUD_ANCHOR_STATE_SUCCESS:
			return TEXT("AR_CLOUD_ANCHOR_STATE_SUCCESS");
		case AR_CLOUD_ANCHOR_STATE_ERROR_INTERNAL:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_INTERNAL");
		case AR_CLOUD_ANCHOR_STATE_ERROR_NOT_AUTHORIZED:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_NOT_AUTHORIZED");
		case AR_CLOUD_ANCHOR_STATE_ERROR_RESOURCE_EXHAUSTED:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_RESOURCE_EXHAUSTED");
		case AR_CLOUD_ANCHOR_STATE_ERROR_HOSTING_DATASET_PROCESSING_FAILED:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_HOSTING_DATASET_PROCESSING_FAILED");
		case AR_CLOUD_ANCHOR_STATE_ERROR_CLOUD_ID_NOT_FOUND:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_CLOUD_ID_NOT_FOUND");
		case AR_CLOUD_ANCHOR_STATE_ERROR_RESOLVING_SDK_VERSION_TOO_OLD:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_RESOLVING_SDK_VERSION_TOO_OLD");
		case AR_CLOUD_ANCHOR_STATE_ERROR_RESOLVING_SDK_VERSION_TOO_NEW:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_RESOLVING_SDK_VERSION_TOO_NEW");
		case AR_CLOUD_ANCHOR_STATE_ERROR_HOSTING_SERVICE_UNAVAILABLE:
			return TEXT("AR_CLOUD_ANCHOR_STATE_ERROR_HOSTING_SERVICE_UNAVAILABLE");
		default:
			return TEXT("AR_CLOUD_ANCHOR_STATE_? Unknown ArCloudAnchorState encountered.");
		}
	}
#endif // ARCORE_SERVICE_SUPPORTED_PLATFORM
} // namespace

#if PLATFORM_ANDROID
class FGoogleARCoreCloudARPinManagerAndroid : FGoogleARCoreCloudARPinManager
{
public:
	FGoogleARCoreCloudARPinManagerAndroid(TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> InArSystem)
		: FGoogleARCoreCloudARPinManager(InArSystem)
	{

	}

	bool IsCloudARPinModeSupported(EARPinCloudMode NewMode) override
	{
		// TODO: we need to figure how to check if config is supported without requesting camera permission.
		return true;
	}

	bool SetCloudARPinMode(EARPinCloudMode NewMode) override
	{
		const UARSessionConfig& Config = ARSystem->GetSessionConfig();
		ArSession* SessionHandle = GetSessionHandle();
		ArConfig* ConfigHandle = nullptr;

		ArConfig_create(SessionHandle, &ConfigHandle);
		ArSession_getConfig(SessionHandle, ConfigHandle);

		ArConfig_setCloudAnchorMode(SessionHandle, ConfigHandle, static_cast<ArCloudAnchorMode>(NewMode));

		ArStatus Status = ArSession_configure(SessionHandle, ConfigHandle);
		ArConfig_destroy(ConfigHandle);

		// TODO: Do we need to surface the error up?
		ensureAlwaysMsgf(Status == AR_SUCCESS, TEXT("Failed to set AR_CLOUD_ANCHOR_MODE_ENABLED in ARCore config"));

		return Status == AR_SUCCESS;
	}

protected:
	ArSession* GetSessionHandle() override
	{
		ArSession* SessionHandle = reinterpret_cast<ArSession*>(ARSystem->GetARSessionRawPointer());
		ensureMsgf(SessionHandle != nullptr, TEXT("Failed to get raw session pointer."));
		return SessionHandle;
	}

	ArFrame* GetARFrameHandle() override
	{
		ArFrame* FrameHandle = reinterpret_cast<ArFrame*>(ARSystem->GetGameThreadARFrameRawPointer());
		ensureMsgf(FrameHandle != nullptr, TEXT("Failed to get raw frame pointer."));
		return FrameHandle;
	}
};
#endif // PLATFORM_ANDROID

#if PLATFORM_IOS
class FGoogleARCoreCloudARPinManageriOS : FGoogleARCoreCloudARPinManager
{
public:
	FGoogleARCoreCloudARPinManageriOS(TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> InArSystem)
		: FGoogleARCoreCloudARPinManager(InArSystem)
		, SessionHandle(nullptr)
		, FrameHandle(nullptr)
	{

	}

	~FGoogleARCoreCloudARPinManageriOS() override
	{
		if (SessionHandle != nullptr)
		{
			ArSession_destroy(SessionHandle);
			SessionHandle = nullptr;
		}
		if (FrameHandle != nullptr)
		{
			ArFrame_release(FrameHandle);
			FrameHandle = nullptr;
		}
	}

	bool IsCloudARPinModeSupported(EARPinCloudMode NewMode) override
	{
		// TODO: we need to figure how to check if config is supported without requesting camera permission.
		return true;
	}

	bool SetCloudARPinMode(EARPinCloudMode NewMode) override
	{
		if (NewMode == EARPinCloudMode::Enabled)
		{
			if (SessionHandle == nullptr)
			{
				FString APIKey = "";
				bool bFoundAPIKey = GConfig->GetString(TEXT("/Script/GoogleARCoreServices.GoogleARCoreServicesEditorSettings"), TEXT("IOSAPIKey"), APIKey, GEngineIni);
				
				// We only create the ARCore iOS session once since ARKit plugin only create the ARKit session once.
				ArStatus Status = ArSession_create(TCHAR_TO_ANSI(*APIKey), nullptr, &SessionHandle);

				static bool ARCoreAnalyticsReported = false;
				if (Status == AR_SUCCESS && !ARCoreAnalyticsReported)
				{
					ArSession_reportEngineType(SessionHandle, "Unreal Engine", TCHAR_TO_ANSI(*FEngineVersion::Current().ToString()));
					ARCoreAnalyticsReported = true;
				}

				switch (Status)
				{
				case AR_SUCCESS:
					break;
				case AR_UNAVAILABLE_DEVICE_NOT_COMPATIBLE:
					UE_LOGF(LogGoogleARCoreServices, Error, "Failed to enable CloudARPin. Device is not compatable.");
					break;
				case AR_ERROR_INVALID_ARGUMENT:
					UE_LOGF(LogGoogleARCoreServices, Error, "Failed to enable CloudARPin. Invalid API Key");
					break;
				default:
					ensureMsgf(false, TEXT("Unhandled error %d in ArSession_create on iOS!"), Status);
					break;
				}

				return Status == AR_SUCCESS;
			}

			return true;
		}
		else if (NewMode == EARPinCloudMode::Disabled)
		{
			// We don't need to do anything but stop update the frame if it the cloud anchor is disabled on iOS.
			return true;
		}

		return false;
	}

protected:
	ArSession* GetSessionHandle() override
	{
		ensureMsgf(SessionHandle != nullptr, TEXT("Failed to get raw session pointer."));
		return SessionHandle;
	}

	ArFrame* GetARFrameHandle() override
	{
		if (FrameHandle != nullptr)
		{
			ArFrame_release(FrameHandle);
			FrameHandle == nullptr;
		}
		ARKitFrame* ArKitFrameHandle = reinterpret_cast<ARKitFrame*>(ARSystem->GetGameThreadARFrameRawPointer());

		ArStatus Status = ArSession_updateAndAcquireArFrame(GetSessionHandle(), ArKitFrameHandle, &FrameHandle);

		ensureMsgf(Status == AR_SUCCESS, TEXT("Failed to update acquire ArFrame from ARKitFrame."));

		return FrameHandle;
	}

private:
	ArSession* SessionHandle;
	ArFrame* FrameHandle;
};
#endif // PLATFORM_IOS

FGoogleARCoreCloudARPinManager* FGoogleARCoreCloudARPinManager::CreateCloudARPinManager(TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> InArSystem)
{
#if PLATFORM_ANDROID
	return reinterpret_cast<FGoogleARCoreCloudARPinManager*>(new FGoogleARCoreCloudARPinManagerAndroid(InArSystem));
#elif PLATFORM_IOS
	return reinterpret_cast<FGoogleARCoreCloudARPinManager*>(new FGoogleARCoreCloudARPinManageriOS(InArSystem));
#else
	return nullptr;
#endif
}

#if ARCORE_SERVICE_SUPPORTED_PLATFORM
#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
void UpdateCloudARPin(UCloudARPin* CloudARPin, ArSession* SessionHandle, ArAnchor* AnchorHandle)
{
	ArCloudAnchorState NewCloudState = AR_CLOUD_ANCHOR_STATE_NONE;
	ArAnchor_getCloudAnchorState(SessionHandle, AnchorHandle, &NewCloudState);
	char* RawCloudID;
	ArAnchor_acquireCloudAnchorId(SessionHandle, AnchorHandle, &RawCloudID);
	FString CloudID(ANSI_TO_TCHAR(RawCloudID));
	CloudARPin->UpdateCloudState(ToARPinCloudState(NewCloudState), CloudID);
	ArString_release(RawCloudID);
} 
#else // ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
void FGoogleARCoreCloudARPinManager::UpdateCloudARPin(UCloudARPin* CloudARPin, ArSession* SessionHandle)
{
	ArFuture* Future = CloudARPin->GetFuture()->AsFuture();
	if (Future == nullptr)
	{
		// We should not call this function on pins that are not pending.
		check(false);
		return;
	}

	ArFutureState FutureState;
	ArFuture_getState(SessionHandle, Future, &FutureState);
	switch (FutureState)
	{
	case AR_FUTURE_STATE_PENDING:
	{
		CloudARPin->UpdateCloudState(ECloudARPinCloudState::InProgress);
		return;
	}
	case AR_FUTURE_STATE_CANCELLED:
	{
		UE_LOGF(LogGoogleARCoreServices, Verbose, "UpdateCloudARPin Future for CloudARPin %ls %ls CANCELLED", *CloudARPin->GetDebugName().ToString(), *CloudARPin->GetCloudID());
		CloudARPin->UpdateCloudState(ECloudARPinCloudState::Cancelled);
		CloudARPin->ReleaseFuture();
		return;
	}
	case AR_FUTURE_STATE_DONE:
	{
		UE_LOGF(LogGoogleARCoreServices, Verbose, "UpdateCloudARPin Future for CloudARPin %ls %ls DONE", *CloudARPin->GetDebugName().ToString(), *CloudARPin->GetCloudID());

		switch (CloudARPin->GetFuture()->GetFutureType())
		{
		case ECloudAnchorFutureType::Host:
		{
			ArHostCloudAnchorFuture* HostFuture = CloudARPin->GetFuture()->AsHostFuture();
			check(HostFuture);
			ArCloudAnchorState CloudAnchorState;
			ArHostCloudAnchorFuture_getResultCloudAnchorState(SessionHandle, HostFuture, &CloudAnchorState);
			CloudARPin->UpdateCloudState(ToARPinCloudState(CloudAnchorState));

			if (CloudAnchorState == AR_CLOUD_ANCHOR_STATE_SUCCESS)
			{
				UE_LOGF(LogGoogleARCoreServices, Verbose, "UpdateCloudARPin Future for Host of CloudARPin %ls (%ls) AR_CLOUD_ANCHOR_STATE_SUCCESS", *CloudARPin->GetDebugName().ToString(), *CloudARPin->GetCloudID());

				char* RawCloudID;
				ArHostCloudAnchorFuture_acquireResultCloudAnchorId(SessionHandle, HostFuture, &RawCloudID);
				CloudARPin->SetCloudID(ANSI_TO_TCHAR(RawCloudID));
				ArString_release(RawCloudID);
			}
			else
			{
				UE_LOGF(LogGoogleARCoreServices, Warning, "UpdateCloudARPin Future for Host of CloudARPin %ls (%ls) not success: %i %ls", *CloudARPin->GetDebugName().ToString(), *CloudARPin->GetCloudID(), (int)CloudAnchorState, ToTCHAR(CloudAnchorState));
			}

			CloudARPin->ReleaseFuture();
			return;
		}
		case ECloudAnchorFutureType::Resolve:
		{
			ArResolveCloudAnchorFuture* ResolveFuture = CloudARPin->GetFuture()->AsResolveFuture();
			check(ResolveFuture);
			ArCloudAnchorState CloudAnchorState;
			ArResolveCloudAnchorFuture_getResultCloudAnchorState(SessionHandle, ResolveFuture, &CloudAnchorState);

			if (CloudAnchorState == AR_CLOUD_ANCHOR_STATE_SUCCESS)
			{
				UE_LOGF(LogGoogleARCoreServices, Verbose, "UpdateCloudARPin Future for Resolve of CloudARPin %ls (%ls) AR_CLOUD_ANCHOR_STATE_SUCCESS", *CloudARPin->GetDebugName().ToString(), *CloudARPin->GetCloudID());

				ArAnchor* AnchorHandle = nullptr;
				ArResolveCloudAnchorFuture_acquireResultAnchor(SessionHandle, ResolveFuture, &AnchorHandle);

				ArPose* PoseHandle = nullptr;
				float WorldToMeterScale = ARSystem->GetXRTrackingSystem()->GetWorldToMetersScale();
				ArPose_create(SessionHandle, nullptr, &PoseHandle);

				ArAnchor_getPose(SessionHandle, AnchorHandle, PoseHandle);

				FTransform PinTrackingTransform = ARCorePoseToUnrealTransform(PoseHandle, SessionHandle, WorldToMeterScale);
				UE_LOGF(LogGoogleARCoreServices, Verbose, "UpdateCloudARPin PinTrackingTransform %ls", *PinTrackingTransform.ToString());
				CloudARPin->InitARPin(ARSystem, nullptr, PinTrackingTransform, nullptr, FName(TEXT("Cloud AR Pin(Acquired)")));
				CloudARPin->UpdateCloudState(ToARPinCloudState(CloudAnchorState));

				HandleToCloudPinMap.Add(AnchorHandle, CloudARPin);

				ArPose_destroy(PoseHandle);
			}
			else
			{
				UE_LOGF(LogGoogleARCoreServices, Warning, "UpdateCloudARPin Future for Resolve of CloudARPin %ls (%ls) not success: %i %ls", *CloudARPin->GetDebugName().ToString(), *CloudARPin->GetCloudID(), (int)CloudAnchorState, ToTCHAR(CloudAnchorState));
			}

			CloudARPin->ReleaseFuture();
			return;
		}
		// Any other supported FutureTypes would need support inserted here.
		default:
			check(false);
			return;
		}
		check(false);
	}
	default:
		check(false);
	}
}
#endif // else ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
#endif // ARCORE_SERVICE_SUPPORTED_PLATFORM

#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
UCloudARPin* FGoogleARCoreCloudARPinManager::CreateAndHostCloudARPin(UARPin* PinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutTaskResult)
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	if (PinToHost == nullptr)
	{
		OutTaskResult = EARPinCloudTaskResult::InvalidPin;
		return nullptr;
	}

	float WorldToMeterScale = ARSystem->GetXRTrackingSystem()->GetWorldToMetersScale();
	ArSession* SessionHandle = GetSessionHandle();
	ArAnchor* NewAnchorHandle = nullptr;
#if PLATFORM_ANDROID
	ArAnchor* AnchorHandle = reinterpret_cast<ArAnchor*> (PinToHost->GetNativeResource());
	ensure(AnchorHandle != nullptr);

	OutTaskResult = ToCloudTaskResult(ArSession_hostAndAcquireNewCloudAnchor(SessionHandle, AnchorHandle, &NewAnchorHandle));
#elif PLATFORM_IOS
	ARKitAnchor* AnchorHandle = nullptr;
	ArPose *PoseHandle = nullptr;
	ArPose_create(SessionHandle, nullptr, &PoseHandle);
	UnrealTransformToARCorePose(PinToHost->GetLocalToTrackingTransform_NoAlignment(), SessionHandle, &PoseHandle, WorldToMeterScale);
	ARKitAnchor_create(PoseHandle, &AnchorHandle);

	ArPose_destroy(PoseHandle);
	PoseHandle = nullptr;

	OutTaskResult = ToCloudTaskResult(ArSession_hostAndAcquireNewCloudAnchor(SessionHandle, AnchorHandle, &NewAnchorHandle));
#endif

	UE_LOGF(LogGoogleARCoreServices, Log, "ArSession_hostAndAcquireNewCloudAnchor returns TaskResult: %u, Anchor: %p", (int)OutTaskResult, NewAnchorHandle);

	UCloudARPin* NewCloudARPin = nullptr;

	if (OutTaskResult == EARPinCloudTaskResult::Started)
	{
		NewCloudARPin = NewObject<UCloudARPin>();
		FTransform PinTrackingTransform = PinToHost->GetLocalToTrackingTransform_NoAlignment();
		NewCloudARPin->InitARPin(ARSystem, nullptr, PinTrackingTransform, PinToHost->GetTrackedGeometry(), FName(TEXT("Cloud AR Pin(Hosted)")));

		UpdateCloudARPin(NewCloudARPin, SessionHandle, NewAnchorHandle);

		AllCloudARPins.Add(NewCloudARPin);
		HandleToCloudPinMap.Add(NewAnchorHandle, NewCloudARPin);
	}

	return NewCloudARPin;
#else
	return nullptr;
#endif
}

UCloudARPin* FGoogleARCoreCloudARPinManager::ResolveAndCreateCloudARPin(FString CloudID, EARPinCloudTaskResult& OutTaskResult)
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	ArSession* SessionHandle = GetSessionHandle();
	ArAnchor* AnchorHandle = nullptr;
	ArPose *PoseHandle = nullptr;
	float WorldToMeterScale = ARSystem->GetXRTrackingSystem()->GetWorldToMetersScale();
	ArPose_create(SessionHandle, nullptr, &PoseHandle);

	OutTaskResult = ToCloudTaskResult(
		ArSession_resolveAndAcquireNewCloudAnchor(SessionHandle, TCHAR_TO_ANSI(*CloudID), &AnchorHandle));
	UE_LOGF(LogGoogleARCoreServices, Log, "ArSession_resolveAndAcquireNewCloudAnchor returns TaskResult: %d, Anchor: %p, CloudID: %ls", (int)OutTaskResult, AnchorHandle, *CloudID);
	ensure(AnchorHandle != nullptr);

	ArAnchor_getPose(SessionHandle, AnchorHandle, PoseHandle);

	UCloudARPin* NewCloudARPin = NewObject<UCloudARPin>();
	FTransform PinTrackingTransform = ARCorePoseToUnrealTransform(PoseHandle, SessionHandle, WorldToMeterScale);
	NewCloudARPin->InitARPin(ARSystem, nullptr, PinTrackingTransform, nullptr, FName(TEXT("Cloud AR Pin(Acquired)")));

	UpdateCloudARPin(NewCloudARPin, SessionHandle, AnchorHandle);

	AllCloudARPins.Add(NewCloudARPin);
	HandleToCloudPinMap.Add(AnchorHandle, NewCloudARPin);

	ArPose_destroy(PoseHandle);

	return NewCloudARPin;
#else
	return nullptr;
#endif
}

#else // ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
UCloudARPin* FGoogleARCoreCloudARPinManager::CreateAndHostCloudARPin(UARPin* PinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutTaskResult)
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	if (PinToHost == nullptr)
	{
		OutTaskResult = EARPinCloudTaskResult::InvalidPin;
		return nullptr;
	}

	float WorldToMeterScale = ARSystem->GetXRTrackingSystem()->GetWorldToMetersScale();
	ArSession* SessionHandle = GetSessionHandle();
#if PLATFORM_ANDROID
	GoogleARFutureHolderPtr FutureHolder = FGoogleARFutureHolder::MakeHostFuture();
	ArAnchor* AnchorHandle = reinterpret_cast<ArAnchor*> (PinToHost->GetNativeResource());
	ensure(AnchorHandle != nullptr);

	ArStatus Result = ArSession_hostCloudAnchorAsync(SessionHandle, AnchorHandle, LifetimeInDays, nullptr, nullptr, FutureHolder->GetHostFuturePtr());
	OutTaskResult = ToCloudTaskResult(Result);
	UE_LOGF(LogGoogleARCoreServices, Verbose, "ArSession_hostCloudAnchorAsync returns TaskResult: %i for Anchor: %p", (int)OutTaskResult, AnchorHandle);

#elif PLATFORM_IOS
	ARKitAnchor* AnchorHandle = nullptr;
	ARKitAnchor* ExistingAnchorHandle = nullptr;
	ArPose* PoseHandle = nullptr;
	ArPose_create(SessionHandle, nullptr, &PoseHandle);
	UnrealTransformToARCorePose(PinToHost->GetLocalToTrackingTransform_NoAlignment(), SessionHandle, &PoseHandle, WorldToMeterScale);
	ARKitAnchor_create(PoseHandle, &ExistingAnchorHandle);

	ArPose_destroy(PoseHandle);
	PoseHandle = nullptr;

	OutTaskResult = ToCloudTaskResult(ArSession_hostAndAcquireNewCloudAnchor(SessionHandle, ExistingAnchorHandle, &AnchorHandle));
	UE_LOGF(LogGoogleARCoreServices, Verbose, "ArSession_hostAndAcquireNewCloudAnchor returns TaskResult: %u, Anchor: %p", (int)OutTaskResult, AnchorHandle);
#endif

	UCloudARPin* NewCloudARPin = nullptr;
	if (OutTaskResult == EARPinCloudTaskResult::Started || OutTaskResult == EARPinCloudTaskResult::Success)
	{
		NewCloudARPin = NewObject<UCloudARPin>();
		FTransform PinTrackingTransform = PinToHost->GetLocalToTrackingTransform_NoAlignment();
		NewCloudARPin->InitARPin(ARSystem, nullptr, PinTrackingTransform, PinToHost->GetTrackedGeometry(), FName(TEXT("Cloud AR Pin(Hosted)")));
		NewCloudARPin->SetFuture(FutureHolder);

		AllCloudARPins.Add(NewCloudARPin);
		PendingCloudARPins.Add(NewCloudARPin);
		HandleToCloudPinMap.Add(AnchorHandle, NewCloudARPin);
	}

	return NewCloudARPin;
#else
	return nullptr;
#endif
}

UCloudARPin* FGoogleARCoreCloudARPinManager::ResolveAndCreateCloudARPin(FString CloudID, EARPinCloudTaskResult& OutTaskResult)
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	ArSession* SessionHandle = GetSessionHandle();

	GoogleARFutureHolderPtr FutureHolder = FGoogleARFutureHolder::MakeResolveFuture();
	ArResolveCloudAnchorFuture* ResolveFutureHandle = nullptr;
	OutTaskResult = ToCloudTaskResult(ArSession_resolveCloudAnchorAsync(SessionHandle, TCHAR_TO_ANSI(*CloudID), nullptr, nullptr, FutureHolder->GetResolveFuturePtr()));

	UCloudARPin* NewCloudARPin = nullptr;
	if (OutTaskResult == EARPinCloudTaskResult::Started || OutTaskResult == EARPinCloudTaskResult::Success)
	{
		NewCloudARPin = NewObject<UCloudARPin>();
		NewCloudARPin->SetFuture(FutureHolder);
		NewCloudARPin->SetCloudID(CloudID);

		AllCloudARPins.Add(NewCloudARPin);
		PendingCloudARPins.Add(NewCloudARPin);
	}

	return NewCloudARPin;
#else
	return nullptr;
#endif
}
#endif // else ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC

void FGoogleARCoreCloudARPinManager::RemoveCloudARPin(UCloudARPin* PinToRemove)
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	auto Key = HandleToCloudPinMap.FindKey(PinToRemove);
	if (Key == nullptr)
	{
		return;
	}
	ArAnchor* AnchorHandle = *Key;

	if (AnchorHandle != nullptr)
	{
		ArAnchor_detach(GetSessionHandle(), AnchorHandle);
		ArAnchor_release(AnchorHandle);

		PinToRemove->OnTrackingStateChanged(EARTrackingState::StoppedTracking);

		HandleToCloudPinMap.Remove(AnchorHandle);
		AllCloudARPins.Remove(PinToRemove);
	}
#endif
}

void FGoogleARCoreCloudARPinManager::Tick()
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	UpdateAllCloudARPins();
#endif
}

void FGoogleARCoreCloudARPinManager::UpdateAllCloudARPins()
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	ArSession* SessionHandle = GetSessionHandle();
	if(SessionHandle == nullptr)
	{
		return;
	}

	ArFrame* FrameHandle = GetARFrameHandle();
	if(FrameHandle == nullptr)
	{
		return;
	}

#if !ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
	// Update any PendingCloudARPins
	for (UCloudARPin* CloudPin : PendingCloudARPins)
	{
		UpdateCloudARPin(CloudPin, SessionHandle);
	}
	// Remove no longer pending CloudARPins
	PendingCloudARPins.RemoveAllSwap([](const UCloudARPin* CloudPin) { return !CloudPin->IsPending(); });
#endif

	float WorldToMeterScale = ARSystem->GetXRTrackingSystem()->GetWorldToMetersScale();

	ArAnchorList* UpdatedAnchorListHandle = nullptr;
	ArAnchorList_create(SessionHandle, &UpdatedAnchorListHandle);
#if PLATFORM_ANDROID
	ArFrame_getUpdatedAnchors(SessionHandle, FrameHandle, UpdatedAnchorListHandle);
#elif PLATFORM_IOS
	ArSession_getAllAnchors(SessionHandle, UpdatedAnchorListHandle);
#endif

	int AnchorListSize = 0;
	ArAnchorList_getSize(SessionHandle, UpdatedAnchorListHandle, &AnchorListSize);

	ArPose* SketchPoseHandle;
	ArPose_create(SessionHandle, nullptr, &SketchPoseHandle);
	for (int i = 0; i < AnchorListSize; i++)
	{
		ArAnchor* AnchorHandle = nullptr;
		ArAnchorList_acquireItem(SessionHandle, UpdatedAnchorListHandle, i, &AnchorHandle);

		ArTrackingState AnchorTrackingState;
		ArAnchor_getTrackingState(SessionHandle, AnchorHandle, &AnchorTrackingState);
		if (!HandleToCloudPinMap.Contains(AnchorHandle))
		{
			continue;
		}

		// Update tracking state and pose
		UCloudARPin* CloudPin = HandleToCloudPinMap[AnchorHandle];
		if (CloudPin->GetTrackingState() != EARTrackingState::StoppedTracking)
		{
			CloudPin->OnTrackingStateChanged(ToARTrackingState(AnchorTrackingState));
		}

		if (CloudPin->GetTrackingState() == EARTrackingState::Tracking)
		{
			ArAnchor_getPose(SessionHandle, AnchorHandle, SketchPoseHandle);
			FTransform AnchorPose = ARCorePoseToUnrealTransform(SketchPoseHandle, SessionHandle, WorldToMeterScale);
			CloudPin->OnTransformUpdated(AnchorPose);
		}

#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
		// Update Cloud state and ID.
		UpdateCloudARPin(CloudPin, SessionHandle, AnchorHandle);
#endif

		ArAnchor_release(AnchorHandle);
	}
	ArAnchorList_destroy(UpdatedAnchorListHandle);
	ArPose_destroy(SketchPoseHandle);
#endif
}
