// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARSupportInterface.h"
#include "GoogleARCoreServicesTypes.h"
#include "GoogleARCoreUtils.h"



class FGoogleARCoreCloudARPinManager : public FGCObject
{

public:
	static FGoogleARCoreCloudARPinManager* CreateCloudARPinManager(TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> InArSystem);

	// public member method:
	virtual bool IsCloudARPinModeSupported(EARPinCloudMode NewMode) = 0;

	virtual bool SetCloudARPinMode(EARPinCloudMode NewMode) = 0;

	// Start a background task to host a CloudARPin.
	virtual UCloudARPin* CreateAndHostCloudARPin(UARPin* PinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutTaskResult);

	// Start a background task to create a new CloudARPin and resolve it from the CloudID.
	virtual UCloudARPin* ResolveAndCreateCloudARPin(FString CloudID, EARPinCloudTaskResult& OutTaskResult);

	virtual void RemoveCloudARPin(UCloudARPin* PinToRemove);

	// Return all the CloudARPin in the current session.
	TArray<UCloudARPin*> GetAllCloudARPin()
	{
		return ObjectPtrDecay(AllCloudARPins);
	}

	// Tick the CloudARPinManager.
	virtual void Tick();

public:
	// public properties:
	virtual ~FGoogleARCoreCloudARPinManager(){ }

protected:
	// protected methods:
	FGoogleARCoreCloudARPinManager(TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> InArSystem)
		: ARSystem(InArSystem)
	{
		ARSystem->OnAlignmentTransformUpdated.AddRaw(this, &FGoogleARCoreCloudARPinManager::OnAlignmentTransformUpdated);
	}

	virtual void UpdateAllCloudARPins();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		for (auto& ARPin : AllCloudARPins)
		{
			Collector.AddReferencedObject(ARPin);
		}
	}

	virtual FString GetReferencerName() const override
	{
		return "GoogleARCoreCloudARPinManager";
	}

	virtual void OnAlignmentTransformUpdated(const FTransform& NewAlignmentTransform)
	{
		for (UCloudARPin* ARPin : AllCloudARPins)
		{
			ARPin->UpdateAlignmentTransform(NewAlignmentTransform);
		}
	}

	// protected properties:
	TSharedRef<FARSupportInterface, ESPMode::ThreadSafe> ARSystem;
	TArray<TObjectPtr<UCloudARPin>> AllCloudARPins;
#if !ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
	TArray<TObjectPtr<UCloudARPin>> PendingCloudARPins; // CloudARPins that have an async action pending.
#endif
#if	ARCORE_SERVICE_SUPPORTED_PLATFORM
#if !ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
	virtual void UpdateCloudARPin(UCloudARPin* CloudARPin, ArSession* SessionHandle);
#endif
	virtual ArSession* GetSessionHandle() = 0;

	virtual ArFrame* GetARFrameHandle() = 0;

	TMap<ArAnchor*, UCloudARPin*> HandleToCloudPinMap;
#endif
};
