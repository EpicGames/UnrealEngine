// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GoogleARCoreServicesEditorSettings.generated.h"

/**
* Helper class used to expose GoogleARCoreServices setting in the Editor plugin settings.
*/
UCLASS(config = Engine, defaultconfig)
class GOOGLEARCORESERVICES_API UGoogleARCoreServicesEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** The API key for GoogleARCoreServices on Android platform. */
	UPROPERTY(EditAnywhere, config, Category = "ARCore Services Plugin Settings", meta = (ShowOnlyInnerProperties))
	FString AndroidAPIKey;

	/** The API key for GoogleARCoreServices on iOS platform. */
	UPROPERTY(EditAnywhere, config, Category = "ARCore Services Plugin Settings", meta = (ShowOnlyInnerProperties))
	FString IOSAPIKey;
};