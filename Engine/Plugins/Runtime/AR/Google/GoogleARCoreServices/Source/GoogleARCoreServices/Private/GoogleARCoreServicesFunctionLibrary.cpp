// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreServicesFunctionLibrary.h"
#include "GoogleARCoreServicesModule.h"
#include "GoogleARCoreServicesManager.h"
#include "Engine/Engine.h"
#include "LatentActions.h"

bool UGoogleARCoreServicesFunctionLibrary::ConfigGoogleARCoreServices(FGoogleARCoreServicesConfig ServiceConfig)
{
	return FGoogleARCoreServicesModule::GetARCoreServicesManager()->ConfigGoogleARCoreServices(ServiceConfig);
}

struct FARCoreServicesHostARPinAction : public FPendingLatentAction
{
public:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	bool bHostStarted;
	UCloudARPin* NewCloudARPin;
	int32 LifetimeInDays;

	UARPin* PinToHost;
	EARPinCloudTaskResult& OutHostingResult;
	UCloudARPin*& OutCloudARPin;

	FARCoreServicesHostARPinAction(const FLatentActionInfo& InLatentInfo, UARPin* InPinToHost, int32 InLifetimeInDays, EARPinCloudTaskResult& InHostingResult, UCloudARPin*& InCloudARPin)
		: FPendingLatentAction()
		, ExecutionFunction(InLatentInfo.ExecutionFunction)
		, OutputLink(InLatentInfo.Linkage)
		, CallbackTarget(InLatentInfo.CallbackTarget)
		, bHostStarted(false)
		, NewCloudARPin(nullptr)
		, LifetimeInDays(InLifetimeInDays)
		, PinToHost(InPinToHost)
		, OutHostingResult(InHostingResult)
		, OutCloudARPin(InCloudARPin)

	{}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		if (!bHostStarted)
		{
			UE_LOGF(LogGoogleARCoreServices, Verbose, "Creating and Hosting CloudARPin.");
			NewCloudARPin = FGoogleARCoreServicesModule::GetARCoreServicesManager()->CreateAndHostCloudARPin(PinToHost, LifetimeInDays, OutHostingResult);
			if (OutHostingResult != EARPinCloudTaskResult::Started)
			{
				// No task is scheduled. Return the task result.
				OutCloudARPin = NewCloudARPin;
				Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
			}
			bHostStarted = true;
		}
		else 
		{
			ECloudARPinCloudState CloudState = NewCloudARPin->GetARPinCloudState();
			if (CloudState == ECloudARPinCloudState::InProgress)
			{
				return;
			}
			if (CloudState == ECloudARPinCloudState::Success && !NewCloudARPin->GetCloudID().IsEmpty())
			{
				OutHostingResult = EARPinCloudTaskResult::Success;
			}
			else
			{
				OutHostingResult = EARPinCloudTaskResult::Failed;
			}
			UE_LOGF(LogGoogleARCoreServices, Verbose, "Creating and Hosting Finished with TaskResult: %d.", (int)OutHostingResult);
			UE_LOGF(LogGoogleARCoreServices, Verbose, "CloudARPin Id: %ls", *NewCloudARPin->GetCloudID());
			OutCloudARPin = NewCloudARPin;
			Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
		}
	}
#if WITH_EDITOR
	virtual FString GetDescription() const override
	{
		return FString::Printf(TEXT("Hosting CloudARPin."));
	}
#endif
};

void UGoogleARCoreServicesFunctionLibrary::CreateAndHostCloudARPinLatentAction(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, UARPin* ARPinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutHostingResult, UCloudARPin*& OutCloudARPin)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		UE_LOGF(LogGoogleARCoreServices, Verbose, "Create Host CloudARPin Action. UUID: %d", LatentInfo.UUID);
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		
		FARCoreServicesHostARPinAction* ExistAction = reinterpret_cast<FARCoreServicesHostARPinAction*>(
			LatentManager.FindExistingAction<FARCoreServicesHostARPinAction>(LatentInfo.CallbackTarget, LatentInfo.UUID));
		if (ExistAction == nullptr || ExistAction->PinToHost != ARPinToHost)
		{
			FARCoreServicesHostARPinAction* NewAction = new FARCoreServicesHostARPinAction(LatentInfo, ARPinToHost, LifetimeInDays, OutHostingResult, OutCloudARPin);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
		else
		{
			UE_LOGF(LogGoogleARCoreServices, Verbose, "Skipping Create Host CloudARPin latent action.");
		}
	}
}

struct FARCoreServicesResolveARPinAction : public FPendingLatentAction
{
public:
	FName ExecutionFunction;
	int32 OutputLink;
	FWeakObjectPtr CallbackTarget;
	bool bAcquireStarted;
	UCloudARPin* NewCloudARPin;

	FString CloudID;
	EARPinCloudTaskResult& OutHostingResult;
	UCloudARPin*& OutCloudARPin;

	FARCoreServicesResolveARPinAction(const FLatentActionInfo& InLatentInfo, FString InCloudID, EARPinCloudTaskResult& InHostingResult, UCloudARPin*& InCloudARPin)
		: FPendingLatentAction()
		, ExecutionFunction(InLatentInfo.ExecutionFunction)
		, OutputLink(InLatentInfo.Linkage)
		, CallbackTarget(InLatentInfo.CallbackTarget)
		, bAcquireStarted(false)
		, NewCloudARPin(nullptr)
		, CloudID(InCloudID)
		, OutHostingResult(InHostingResult)
		, OutCloudARPin(InCloudARPin)
	{}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		if (!bAcquireStarted)
		{
			NewCloudARPin = FGoogleARCoreServicesModule::GetARCoreServicesManager()->ResolveAncCreateCloudARPin(CloudID, OutHostingResult);
			if (OutHostingResult != EARPinCloudTaskResult::Started)
			{
				// No task is scheduled. Return the task result.
				OutCloudARPin = NewCloudARPin;
				Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
			}
			bAcquireStarted = true;
		}
		else
		{
			ECloudARPinCloudState CloudState = NewCloudARPin->GetARPinCloudState();
			if (CloudState == ECloudARPinCloudState::InProgress)
			{
				return;
			}
			if (CloudState == ECloudARPinCloudState::Success && !NewCloudARPin->GetCloudID().IsEmpty())
			{
				OutHostingResult = EARPinCloudTaskResult::Success;
			}
			else
			{
				OutHostingResult = EARPinCloudTaskResult::Failed;
			}
			OutCloudARPin = NewCloudARPin;
			Response.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
		}
	}
#if WITH_EDITOR
	virtual FString GetDescription() const override
	{
		return FString::Printf(TEXT("Resolve CloudARPin."));
	}
#endif
};

void UGoogleARCoreServicesFunctionLibrary::CreateAndResolveCloudARPinLatentAction(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, FString CloudId, EARPinCloudTaskResult& OutAcquiringResult, UCloudARPin*& OutCloudARPin)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FARCoreServicesResolveARPinAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FARCoreServicesResolveARPinAction* NewAction = new FARCoreServicesResolveARPinAction(LatentInfo, CloudId, OutAcquiringResult, OutCloudARPin);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
		else
		{
			UE_LOGF(LogGoogleARCoreServices, Log, "Skipping Create Host CloudARPin latent action %d.", LatentInfo.UUID);
		}
	}
}

UCloudARPin* UGoogleARCoreServicesFunctionLibrary::CreateAndHostCloudARPin(UARPin* ARPinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutTaskResult)
{
	return FGoogleARCoreServicesModule::GetARCoreServicesManager()->CreateAndHostCloudARPin(ARPinToHost, LifetimeInDays, OutTaskResult);
}

UCloudARPin* UGoogleARCoreServicesFunctionLibrary::CreateAndResolveCloudARPin(FString CloudId, EARPinCloudTaskResult& OutTaskResult)
{
	return FGoogleARCoreServicesModule::GetARCoreServicesManager()->ResolveAncCreateCloudARPin(CloudId, OutTaskResult);
}

void UGoogleARCoreServicesFunctionLibrary::RemoveCloudARPin(UCloudARPin* PinToRemove)
{
	FGoogleARCoreServicesModule::GetARCoreServicesManager()->RemoveCloudARPin(PinToRemove);
}

TArray<UCloudARPin*> UGoogleARCoreServicesFunctionLibrary::GetAllCloudARPin()
{
	return FGoogleARCoreServicesModule::GetARCoreServicesManager()->GetAllCloudARPin();
}
