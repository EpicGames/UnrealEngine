// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreServicesManager.h"
#include "Engine/Engine.h"
#include "GoogleARCoreCloudARPinManager.h"
#include "XRTrackingSystemBase.h"

FGoogleARCoreServicesManager::FGoogleARCoreServicesManager()
	: bHasValidARSystem(false)
	, bCloudARPinEnabled(false)
	, ARSystem(nullptr)
	, CloudARPinManager(nullptr)
{
	CurrentServicesConfig.ARPinCloudMode = EARPinCloudMode::Disabled;
}

FGoogleARCoreServicesManager::~FGoogleARCoreServicesManager()
{
	if (bHasValidARSystem)
	{
		FWorldDelegates::OnWorldTickStart.RemoveAll(this);
	}
}

bool FGoogleARCoreServicesManager::ConfigGoogleARCoreServices(FGoogleARCoreServicesConfig& ServiceConfig)
{
	if (!bHasValidARSystem)
	{
		if (InitARSystem())
		{
			bHasValidARSystem = true;
		}
		else
		{
			return false;
		}
	}

	CurrentServicesConfig = ServiceConfig;

	if (bHasValidARSystem && ARSystem->GetARSessionStatus().Status != EARSessionStatus::Running)
	{
		// Defer the configuration to the next session started.
		return CloudARPinManager->IsCloudARPinModeSupported(CurrentServicesConfig.ARPinCloudMode);
	}

	bool bConfigSucceed = CloudARPinManager->SetCloudARPinMode(CurrentServicesConfig.ARPinCloudMode);
	if (bConfigSucceed)
	{
		bCloudARPinEnabled = CurrentServicesConfig.ARPinCloudMode == EARPinCloudMode::Enabled;
	}

	return bConfigSucceed;
}

UCloudARPin* FGoogleARCoreServicesManager::CreateAndHostCloudARPin(UARPin* ARPinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutTaskResult)
{
	OutTaskResult = CheckCloudTaskError();
	if (OutTaskResult != EARPinCloudTaskResult::Success)
	{
		return nullptr;
	}

	UCloudARPin* NewCloudARPin = CloudARPinManager->CreateAndHostCloudARPin(ARPinToHost, LifetimeInDays, OutTaskResult);

	return NewCloudARPin;
}

UCloudARPin* FGoogleARCoreServicesManager::ResolveAncCreateCloudARPin(FString CloudId, EARPinCloudTaskResult& OutTaskResult)
{
	OutTaskResult = CheckCloudTaskError();
	if (OutTaskResult != EARPinCloudTaskResult::Success)
	{
		return nullptr;
	}

	UCloudARPin* NewCloudARPin = CloudARPinManager->ResolveAndCreateCloudARPin(CloudId, OutTaskResult);

	return NewCloudARPin;
}

void FGoogleARCoreServicesManager::RemoveCloudARPin(UCloudARPin* PinToRemove)
{
	if (bHasValidARSystem)
	{
		CloudARPinManager->RemoveCloudARPin(PinToRemove);
	}
}

TArray<UCloudARPin*> FGoogleARCoreServicesManager::GetAllCloudARPin()
{
	if (bHasValidARSystem)
	{
		return CloudARPinManager->GetAllCloudARPin();
	}

	return TArray<UCloudARPin*>();
}

bool FGoogleARCoreServicesManager::InitARSystem()
{
	ARSystem = StaticCastSharedPtr<FXRTrackingSystemBase>(GEngine->XRSystem)->GetARCompositionComponent();

	if (ARSystem.IsValid())
	{
		ARSystem->OnARSessionStarted.AddRaw(this, &FGoogleARCoreServicesManager::OnARSessionStarted);

		CloudARPinManager = TUniquePtr<FGoogleARCoreCloudARPinManager>(FGoogleARCoreCloudARPinManager::CreateCloudARPinManager(ARSystem.ToSharedRef()));
		FWorldDelegates::OnWorldTickStart.AddRaw(this, &FGoogleARCoreServicesManager::OnWorldTickStart);
		return true;
	}
	else
	{
		UE_LOGF(LogGoogleARCoreServices, Log, "No valid ARSystem is found. GoogleARCoreServices will be disabled.");
		return false;
	}
}

EARPinCloudTaskResult FGoogleARCoreServicesManager::CheckCloudTaskError()
{
	if (!bHasValidARSystem || !bCloudARPinEnabled)
	{
		return EARPinCloudTaskResult::CloudARPinNotEnabled;
	}

	if (ARSystem->GetARSessionStatus().Status != EARSessionStatus::Running)
	{
		return EARPinCloudTaskResult::SessionPaused;
	}

	if (ARSystem->GetTrackingQuality() != EARTrackingQuality::OrientationAndPosition)
	{
		return EARPinCloudTaskResult::NotTracking;
	}
	return EARPinCloudTaskResult::Success;
}

void FGoogleARCoreServicesManager::OnARSessionStarted()
{
	bCloudARPinEnabled = CloudARPinManager->SetCloudARPinMode(CurrentServicesConfig.ARPinCloudMode);
}

// We need to make sure this tick happens after the ARSystem tick
void FGoogleARCoreServicesManager::OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaTime)
{
	if (!bHasValidARSystem)
	{
		return;
	}

	if(bCloudARPinEnabled && ARSystem->GetARSessionStatus().Status == EARSessionStatus::Running)
	{
		CloudARPinManager->Tick();
	}
}

