// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/EngineBaseTypes.h"
#include "GoogleARCoreServicesTypes.h"

class FGoogleARCoreCloudARPinManager;
class UARPin;

class FGoogleARCoreServicesManager
{

public:
	FGoogleARCoreServicesManager();

	~FGoogleARCoreServicesManager();

	bool ConfigGoogleARCoreServices(FGoogleARCoreServicesConfig& ServiceConfig);

	UCloudARPin* CreateAndHostCloudARPin(UARPin* ARPinToHost, int32 InLifetimeInDays, EARPinCloudTaskResult& OutTaskResult);

	UCloudARPin* ResolveAncCreateCloudARPin(FString CloudId, EARPinCloudTaskResult& OutTaskResult);

	void RemoveCloudARPin(UCloudARPin* PinToRemove);

	TArray<UCloudARPin*> GetAllCloudARPin();

private:
	bool InitARSystem();

	EARPinCloudTaskResult CheckCloudTaskError();

	void OnARSessionStarted();
	void OnWorldTickStart(UWorld* World, ELevelTick TickType, float DeltaTime);

	bool bHasValidARSystem;
	bool bCloudARPinEnabled;
	FGoogleARCoreServicesConfig CurrentServicesConfig;

	TSharedPtr<FARSupportInterface, ESPMode::ThreadSafe> ARSystem;
	TUniquePtr<FGoogleARCoreCloudARPinManager> CloudARPinManager;
};

