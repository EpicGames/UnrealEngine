// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreServicesModule.h"
#include "GoogleARCoreServicesManager.h"
#include "GoogleARCoreServicesEditorSettings.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FGoogleARCoreServicesModule"

static TUniquePtr<FGoogleARCoreServicesManager> ARCoreServicesManager = nullptr;

class FGoogleARCoreServicesManager* FGoogleARCoreServicesModule::GetARCoreServicesManager()
{
	check(ARCoreServicesManager.IsValid());
	return ARCoreServicesManager.Get();
}

void FGoogleARCoreServicesModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	ARCoreServicesManager = MakeUnique<FGoogleARCoreServicesManager>();

	// Register editor settings:
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "GoogleARCoreServices",
			LOCTEXT("GoogleARCoreServicesSetting", "GoogleARCoreServices"),
			LOCTEXT("GoogleARCoreServicesSettingDescription", "Settings of the GoogleARCoreServices plugin"),
			GetMutableDefault<UGoogleARCoreServicesEditorSettings>());
	}
}

void FGoogleARCoreServicesModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	ARCoreServicesManager.Reset();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGoogleARCoreServicesModule, GoogleARCoreServices)
