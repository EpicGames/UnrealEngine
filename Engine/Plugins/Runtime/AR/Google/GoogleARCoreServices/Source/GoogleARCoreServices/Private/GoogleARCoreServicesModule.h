// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FGoogleARCoreServicesManager;
class FGoogleARCoreServicesModule : public IModuleInterface
{
public:

	// This function will always return a valid FGoogleARCoreServicesManager.
	// Or it will cause a crash if the function is called before the module startup.
	static FGoogleARCoreServicesManager* GetARCoreServicesManager();

	/** IModuleInterface implementation. */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

};
