// Copyright Epic Games, Inc. All Rights Reserved.

#include "GoogleARCoreServicesTypes.h"
#include "GoogleARCoreUtils.h"

#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
UCloudARPin::UCloudARPin()
	: UARPin()
{
	CloudState = ECloudARPinCloudState::NotHosted;
	CloudID = FString("");
}

FString UCloudARPin::GetCloudID()
{
	return CloudID;
}

ECloudARPinCloudState UCloudARPin::GetARPinCloudState()
{
	return CloudState;
}

void UCloudARPin::UpdateCloudState(ECloudARPinCloudState NewCloudState, FString NewCloudID)
{
	CloudState = NewCloudState;
	CloudID = NewCloudID;
}

#else //ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC

GoogleARFutureHolderPtr FGoogleARFutureHolder::MakeHostFuture()
{
	return MakeShared<FGoogleARFutureHolder>(ECloudAnchorFutureType::Host);
}
GoogleARFutureHolderPtr FGoogleARFutureHolder::MakeResolveFuture()
{
	return MakeShared<FGoogleARFutureHolder>(ECloudAnchorFutureType::Resolve);
}

FGoogleARFutureHolder::FGoogleARFutureHolder(ECloudAnchorFutureType InFutureType)
	: FutureType(InFutureType)
	, Future(nullptr)
{
}

FGoogleARFutureHolder::~FGoogleARFutureHolder()
{
#if ARCORE_SERVICE_SUPPORTED_PLATFORM
	if (Future != nullptr)
	{
		ArFuture_release(Future);
		Future = nullptr;
	}
#endif
}

ArHostCloudAnchorFuture** FGoogleARFutureHolder::GetHostFuturePtr()
{
	check(FutureType == ECloudAnchorFutureType::Host);
	return &HostFuture;
}
ArResolveCloudAnchorFuture** FGoogleARFutureHolder::GetResolveFuturePtr()
{
	check(FutureType == ECloudAnchorFutureType::Resolve);
	return &ResolveFuture;
}

ECloudAnchorFutureType FGoogleARFutureHolder::GetFutureType() const
{
	return FutureType;
}

ArFuture* const FGoogleARFutureHolder::AsFuture() const
{
	return Future;
}
ArHostCloudAnchorFuture* const FGoogleARFutureHolder::AsHostFuture() const
{
	return FutureType == ECloudAnchorFutureType::Host ? HostFuture : nullptr;
}
ArResolveCloudAnchorFuture* const FGoogleARFutureHolder::AsResolveFuture() const
{
	return FutureType == ECloudAnchorFutureType::Resolve ? ResolveFuture : nullptr;
}

UCloudARPin::UCloudARPin()
	: UARPin()
	, CloudState(ECloudARPinCloudState::NotHosted)
{
	CloudID = FString("");
}

FString UCloudARPin::GetCloudID()
{
	return CloudID;
}

ECloudARPinCloudState UCloudARPin::GetARPinCloudState()
{
	return CloudState;
}

void UCloudARPin::SetFuture(GoogleARFutureHolderPtr InFuture)
{
	Future = InFuture;
}

GoogleARFutureHolderPtr UCloudARPin::GetFuture() const
{
	return Future;
}

void UCloudARPin::UpdateCloudState(ECloudARPinCloudState NewCloudState)
{
	CloudState = NewCloudState;
}

void UCloudARPin::SetCloudID(FString NewCloudID)
{
	CloudID = NewCloudID;
}

bool UCloudARPin::IsPending() const
{
	return Future.IsValid();
}

void UCloudARPin::ReleaseFuture()
{
	Future = nullptr;
}
#endif
