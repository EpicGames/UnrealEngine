// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_ANDROID || PLATFORM_IOS
#define ARCORE_SERVICE_SUPPORTED_PLATFORM 1
#else
#define ARCORE_SERVICE_SUPPORTED_PLATFORM 0
#endif

#if ARCORE_SERVICE_SUPPORTED_PLATFORM

#include "ARTypes.h"

#if PLATFORM_ANDROID
#include "arcore_c_api.h"
#elif PLATFORM_IOS
#include "arcore_ios_c_api.h"
#endif

namespace
{
EARTrackingState ToARTrackingState(ArTrackingState State)
{
	switch (State)
	{
	case AR_TRACKING_STATE_PAUSED:
		return EARTrackingState::NotTracking;
	case AR_TRACKING_STATE_STOPPED:
		return EARTrackingState::StoppedTracking;
	case AR_TRACKING_STATE_TRACKING:
		return EARTrackingState::Tracking;
	}
}

const FMatrix ARCoreToUnrealTransform = FMatrix(
	FPlane(0.0f, 0.0f, -1.0f, 0.0f),
	FPlane(1.0f, 0.0f, 0.0f, 0.0f),
	FPlane(0.0f, 1.0f, 0.0f, 0.0f),
	FPlane(0.0f, 0.0f, 0.0f, 1.0f));

const FMatrix ARCoreToUnrealTransformInverse = ARCoreToUnrealTransform.InverseFast();

FTransform ARCorePoseToUnrealTransform(ArPose* ArPoseHandle, const ArSession* SessionHandle, float WorldToMeterScale)
{
	FMatrix44f ARCorePoseMatrix;
	ArPose_getMatrix(SessionHandle, ArPoseHandle, ARCorePoseMatrix.M[0]);
	FTransform Result = FTransform(ARCoreToUnrealTransform * FMatrix(ARCorePoseMatrix) * ARCoreToUnrealTransformInverse);
	Result.SetLocation(Result.GetLocation() * WorldToMeterScale);

	return Result;
}

void UnrealTransformToARCorePose(const FTransform& UnrealTransform, const ArSession* SessionHandle, ArPose** OutARPose, float WorldToMeterScale)
{
	check(OutARPose);

	FMatrix UnrealPoseMatrix = UnrealTransform.ToMatrixNoScale();
	UnrealPoseMatrix.SetOrigin(UnrealPoseMatrix.GetOrigin() / WorldToMeterScale);
	FMatrix ARCorePoseMatrix = ARCoreToUnrealTransformInverse * UnrealPoseMatrix * ARCoreToUnrealTransform;

	FVector ArPosePosition = ARCorePoseMatrix.GetOrigin();
	FQuat ArPoseRotation = ARCorePoseMatrix.ToQuat();
	float ArPoseData[7] = { (float)ArPoseRotation.X, (float)ArPoseRotation.Y, (float)ArPoseRotation.Z, (float)ArPoseRotation.W, (float)ArPosePosition.X, (float)ArPosePosition.Y, (float)ArPosePosition.Z };	// LWC_TODO: Precision loss
	ArPose_create(SessionHandle, ArPoseData, OutARPose);
}

} // namespace
#endif // ARCORE_SERVICE_SUPPORTED_PLATFORM
