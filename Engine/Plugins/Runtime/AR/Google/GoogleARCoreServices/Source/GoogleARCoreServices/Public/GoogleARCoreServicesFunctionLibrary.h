// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/LatentActionManager.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "ARPin.h"
#include "GoogleARCoreServicesTypes.h"

#include "GoogleARCoreServicesFunctionLibrary.generated.h"

/** A function library that provides static/Blueprint functions for Google ARCore Services.*/
UCLASS()
class GOOGLEARCORESERVICES_API UGoogleARCoreServicesFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Configure the current Unreal AR session with the desired GoogleARCoreServices configuration.
	 * If there is an running AR session, the configuration will take effect immediately. Otherwise,
	 * it will take effect when the next AR session is running.
	 * 
	 * @param ServiceConfig	 The desired GoogleARCoreServices configuration.
	 * @return               True if GoogleARCoreServices is configured successfully.
	 *                       False if the configuration failed to apply.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|Configuration", meta = (Keywords = "googlear ar service config"))
	static bool ConfigGoogleARCoreServices(FGoogleARCoreServicesConfig ServiceConfig);

	/**
	 * This will start a Latent Action to host the ARPin and creating a UCloudARPin from it.
	 * The complete flow of this Latent Action will be triggered if the hosting is complete
	 * or an error has occurred.
	 *
	 * Note that a UCloudARPin will be always created when this function is called, even in the case
	 * that the CloudId is failed to host. You can check the CloudState of returning UCloudARPin
	 * to see why the hosting failed.
	 *
	 * @param ARPinToHost       The ARPin to host.
	 * @param LifetimeInDays	The lifetime of the cloud anchor in days.
	 * @param OutHostingResult  The ARPin hosting result.
	 * @param OutCloudARPin     A new instance of UCloudARPin created using the input ARPinToHost.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|CloudARPin", meta = (Latent, LatentInfo = "LatentInfo", LifetimeInDays = 1, WorldContext = "WorldContextObject", Keywords = "googlear ar service host cloud"))
	static void CreateAndHostCloudARPinLatentAction(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, UARPin* ARPinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutHostingResult, UCloudARPin*& OutCloudARPin);

	/**
	 * This will start a Latent Action to create UCloudARPin using the given CloudId. The complete flow
	 * of this Latent Action will be triggered if creating the UCloudARPin is successfully or an error
	 * has occurred.
	 *
	 * Note that a UCloudARPin will be always created when this function is called, even in the case
	 * that the CloudId is failed to resolve. You can check the CloudState of returning UCloudARPin
	 * to see why the resolving failed.
	 *
	 * @param CloudId               The CloudId that will be used to resolve the ARPin
	 * @param OutAcquiringResult    The ARPin acquiring result.
	 * @param OutARPin              The ARPin that is created when calling this function. 
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|CloudARPin", meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject", Keywords = "googlear ar service resolve cloud"))
	static void CreateAndResolveCloudARPinLatentAction(UObject* WorldContextObject, struct FLatentActionInfo LatentInfo, FString CloudId, EARPinCloudTaskResult& OutAcquiringResult, UCloudARPin*& OutCloudARPin);

	/**
	 * Creating and hosting a CloudARPin and return it immediately.
	 * Note that this function only start the hosting process. Call GetARPinCloudState to check 
	 * if the hosting is finished or failed with error.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|CloudARPin", meta = (LifetimeInDays = 1, Keywords = "googlear ar service remove "))
	static UCloudARPin* CreateAndHostCloudARPin(UARPin* ARPinToHost, int32 LifetimeInDays, EARPinCloudTaskResult& OutTaskResult);

	/**
	 * Creating and Resolving a CloudARPin and return it immediately.
	 * Note that this function only start the acquiring process. Call GetARPinCloudState to check
	 * if the acquiring is finished or failed with error.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|CloudARPin", meta = (Keywords = "googlear ar service remove "))
	static UCloudARPin* CreateAndResolveCloudARPin(FString CloudId, EARPinCloudTaskResult& OutTaskResult);

	/**
	 * Remove the given CloudARPin from the current ARSession.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|CloudARPin", meta = (Keywords = "googlear ar service remove "))
	static void RemoveCloudARPin(UCloudARPin* PinToRemove);

	/**
	 * Get a list of all CloudARPin in the current ARSession.
	 */
	UFUNCTION(BlueprintCallable, Category = "GoogleARCoreServices|CloudARPin", meta = (Keywords = "googlear ar service all"))
	static TArray<UCloudARPin*> GetAllCloudARPin();
};