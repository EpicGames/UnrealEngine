// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ARPin.h"
#include "GoogleARCoreServicesTypes.generated.h"

DEFINE_LOG_CATEGORY_STATIC(LogGoogleARCoreServices, Log, All);


#if PLATFORM_IOS
#define ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC 1
#else
#define ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC 0
#endif

#if !ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
typedef struct ArFuture_ ArFuture;
typedef struct ArHostCloudAnchorFuture_ ArHostCloudAnchorFuture;
typedef struct ArResolveCloudAnchorFuture_ ArResolveCloudAnchorFuture;
#endif

/// @defgroup GoogleARCoreServices Google ARCore Services
/// The module for GoogleARCoreServices plugin

/** 
 * @ingroup GoogleARCoreServices
 * This is an enum that can be set in a FGoogleARCoreServicesConfig to enable/disable 
 * cloud ARPin. When EARPinCloudMode is Enabled, ARCoreServices will provides functionalities
 * of hosting and resolving cloud ARPins, with the overhead of maintaining the rolling
 * buffer of feature/IMU measurements, and the requirements of INTERNET permissions.
 */
UENUM(BlueprintType, Category = "GoogleARCoreServices|Configuration")
enum class EARPinCloudMode : uint8
{
	Disabled = 0,
	Enabled = 1
};

/**
 * @ingroup GoogleARCoreServices
 * A struct describes the configuration in GoogleARCore Services
 */
USTRUCT(BlueprintType, Category = "GoogleARCoreServices|Configuration")
struct FGoogleARCoreServicesConfig
{
	GENERATED_BODY()

	/** Whether enabling ARPin hosting/resolving in GoogleARCoreServices. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GoogleARCoreServices|Configuration")
	EARPinCloudMode ARPinCloudMode = EARPinCloudMode::Disabled;
};

/**
 * @ingroup GoogleARCoreServices
 * A enum describes the CloudARPin task result.
 */
UENUM(BlueprintType, Category = "GoogleARCoreServices|CloudARPin")
enum class EARPinCloudTaskResult : uint8
{
	// CloudARPin hosting/resolving task is successfully completed.
	// Only used the latent action UGoogleARCoreServicesFunctionLibrary::CreateAndHostCloudARPinLatentAction
	// and UGoogleARCoreServicesFunctionLibrary::CreateAndResolveCloudARPinLatentAction.
	Success,
	// CloudARPin hosting/resolving is failed. Check the CloudARPin cloud state for details why it failed.
	// Only used the latent action UGoogleARCoreServicesFunctionLibrary::CreateAndHostCloudARPinLatentAction
	// and UGoogleARCoreServicesFunctionLibrary::CreateAndResolveCloudARPinLatentAction.
	Failed,
	// CloudARPin hosting/resolving is started successfully. 
	// Only used the non-latent UGoogleARCoreServicesFunctionLibrary::HostARPin and
	// UGoogleARCoreServicesFunctionLibrary::AcquireHostedARPin.
	Started,
	// CloudARPin hosting/resolving failed because ARPin hosting isn't enabled.
	CloudARPinNotEnabled,
	// CloudARPin hosting/resolving failed because ARPin is not in Tracking State.
	NotTracking,
	// CloudARPin hosting/resolving failed because there is no valid ARSession or the session is paused.
	SessionPaused,
	// CloudARPin hosting failed because the input ARPin is invalid.
	InvalidPin,
	// CloudARPin hosting failed because a new CloudARPin couldn't be created in ARSystem due to 
	// resource exhausted.
	ResourceExhausted
};

/**
 * @ingroup GoogleARCoreServices
 * A enum describes the cloud state of a CloudARPin.
 */
UENUM(BlueprintType, Category = "GoogleARCoreServices|CloudARPin")
enum class ECloudARPinCloudState : uint8
{
	// The CloudARPin just got created and the background task for hosting/resolving the CloudARPin
	// hasn't started yet.
	NotHosted,

	// A hosting/resolving task for the CloudARPin has been scheduled.
	// Once the task completes in the background, the CloudARPin will get
	// a new cloud state in the next frame.
	// NOTE: this does not correspond to an ArCloudAnchorState, this value results with the ArFuture is pending.
	InProgress,

	// A hosting/resolving task has been cancelled.  This CloudARPin would not be cloud hosted.
	// NOTE: this does not correspond to an ArCloudAnchorState, this value results with the ArFuture is canccelled.
	Cancelled,

	// A hosting/resolving task for this CloudARPin completed successfully.
	Success,

	// A hosting/resolving task for this CloudARPin finished with an internal error.
	// This error is hard to recover from, and there is likely nothing that the
	// developer can do to mitigate it.
	ErrorInternalError,

	// The app cannot communicate with the Google AR Cloud Service because of a
	// bad/invalid/nonexistent API key in the manifest.
	ErrorNotAuthorized,

	// The server could not localize the device for the requested Cloud ID. This
	// means that the ARPin was not present in the user's surroundings.
	ErrorLocalizationFailure,

	// The Google AR Cloud Service was unreachable. This can happen because of a
	// number of reasons. The request sent to the server could have timed out
	// with no response, there could be a bad network connection, DNS
	// unavailability, firewall issues, or anything that could affect the
	// device's ability to connect to the Google AR cloud service.
	ErrorServiceUnavailable,

	// The application has exhausted the request quota allotted to the given API
	// key. The developer should request more quota for the Google AR Cloud
	// Service for their API key from the Google Developer Console.
	ErrorResourceExhausted,

	// Hosting failed, because the server could not successfully process the
	// dataset for the given anchor. The developer should try again after the
	// device has gathered more data from the environment.
	ErrorHostingDatasetProcessingFailed,

	// Resolving failed, because the AR Cloud Service could not find the
	// provided cloud anchor ID.
	ErrorResolvingCloudIDNotFound,

	// The CloudARPin could not be resolved because the SDK used to host the CloudARPin
	// was newer than the version being used to acquire it. These versions must
	// be an exact match.
	ErrorSDKVersionTooOld,

	// The CloudARPin could not be acquired because the SDK used to host the CloudARPin
	// was older than the version being used to acquire it. These versions must
	// be an exact match.
	ErrorSDKVersionTooNew
};

#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
#else
enum class ECloudAnchorFutureType : uint8
{
	Unknown,
	Host,
	Resolve
};

// An raii holder for ArFutures of the types we use, to ensure they are released.
typedef TSharedPtr<struct FGoogleARFutureHolder> GoogleARFutureHolderPtr;
struct FGoogleARFutureHolder
{
	static GoogleARFutureHolderPtr MakeHostFuture();
	static GoogleARFutureHolderPtr MakeResolveFuture();

	FGoogleARFutureHolder(ECloudAnchorFutureType InFutureType);
	~FGoogleARFutureHolder();

	ArHostCloudAnchorFuture** GetHostFuturePtr();
	ArResolveCloudAnchorFuture** GetResolveFuturePtr();
	ECloudAnchorFutureType GetFutureType() const;
	ArFuture* const AsFuture() const;
	ArHostCloudAnchorFuture* const AsHostFuture() const;
	ArResolveCloudAnchorFuture* const AsResolveFuture() const;

private:
	const ECloudAnchorFutureType FutureType;
	union
	{
		ArFuture* Future;
		ArHostCloudAnchorFuture* HostFuture;
		ArResolveCloudAnchorFuture* ResolveFuture;
	};
};
#endif // !ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC

/**
 * A CloudARPin will be created when you host an existing ARPin, or resolved a
 * previous hosted CloudARPin. It is a subclass of UARPin so all functions on 
 * UARPin works on CloudARPin, besides that you can query its CloudState and CloudID
 */
UCLASS(BlueprintType, Experimental, Category = "GoogleARCoreServices|CloudARPin")
class GOOGLEARCORESERVICES_API UCloudARPin : public UARPin
{
	GENERATED_BODY()

public:

	UCloudARPin();

	/**
	 * Gets the CloudID of this CloudARPin.
	 * @return	Return a non-empty FString if the CloudARPin is ready. otherwise, return
	 *          an empty FString.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCoreServices|CloudARPin")
	FString GetCloudID();

	/**
	 * Gets the current cloud state of this CloudARPin.
	 * Note that the cloud state will only be updated once per frame.
	 */
	UFUNCTION(BlueprintPure, Category = "GoogleARCoreServices|CloudARPin")
	ECloudARPinCloudState GetARPinCloudState();

#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
	void UpdateCloudState(ECloudARPinCloudState NewCloudState, FString NewCloudID);
#else
	void SetFuture(GoogleARFutureHolderPtr FutureHolder);
	GoogleARFutureHolderPtr GetFuture() const;
	bool IsPending() const;
	void ReleaseFuture();

	void UpdateCloudState(ECloudARPinCloudState NewCloudState);

	void SetCloudID(FString NewCloudID);
	#endif

private:
	ECloudARPinCloudState CloudState;
	FString CloudID;
#if ARCORE_USE_OLD_CLOUD_ANCHOR_ASYNC
#else
	GoogleARFutureHolderPtr Future;
#endif
};
